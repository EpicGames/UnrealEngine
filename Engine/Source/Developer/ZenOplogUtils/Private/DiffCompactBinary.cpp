// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Experimental/DiffCompactBinary.h"
#include "Misc/StringBuilder.h"

namespace UE
{
	static constexpr TCHAR GCbDelimiter = '/';

	void ReportCompactBinaryDifferences(TArray<FUtf8StringView>& PropertyNameStack, FCbFieldView Entry1, FCbFieldView Entry2, TArray<FCbEntryDifference>& OutResults);

	FUtf8String CreatePropertyHierarchicalName(const TArray<FUtf8StringView>& PropertyNameStack, FUtf8StringView PropertyName = {})
	{
		TUtf8StringBuilder<256> NameBuilder;
		for (const FUtf8StringView& Property : PropertyNameStack)
		{
			NameBuilder.Append(Property);
			NameBuilder.Add(GCbDelimiter);
		}
		if (PropertyName.Len() == 0 && NameBuilder.Len() > 0)	// Remove the final separator if there is no suffix 
		{
			NameBuilder.RemoveSuffix(1);
		}
		else
		{
			NameBuilder.Append(PropertyName);
		}
		return NameBuilder.ToString();
	}

	void ReportCompactBinaryDifferences(TArray<FUtf8StringView>& PropertyNameStack, FCbObjectView Entry1, FCbObjectView Entry2, TArray<FCbEntryDifference>& OutResults)
	{
		// Reduce the number of field lookups required by caching them once during key lookup
		struct FCachedObjectFields
		{
			FCbFieldView Entry1;
			FCbFieldView Entry2;
		};
		TArray<FUtf8StringView, TInlineAllocator<16>> AllKeys;			// Cache all keys found in both objects
		TArray<FCachedObjectFields, TInlineAllocator<16>> AllFields;		// Cache the fields associated with the above keys
		for (const FCbFieldView EntryChild : Entry1)
		{
			if (EntryChild.HasName())
			{
				AllKeys.Add(EntryChild.GetName());
				AllFields.Add(FCachedObjectFields
				{
					.Entry1 = EntryChild
				});
			}
		}
		for (const FCbFieldView EntryChild : Entry2)
		{
			if (EntryChild.HasName())
			{
				int32 NewIndex = AllKeys.AddUnique(EntryChild.GetName());
				if (NewIndex >= AllFields.Num())	// New entry needed, this field is not in object1
				{
					AllFields.Emplace();
				}
				AllFields[NewIndex].Entry2 = EntryChild;
			}
		}

		for (int32 FieldIndex = 0; FieldIndex < AllFields.Num(); ++FieldIndex)
		{
			const FCbFieldView& Entry1Child = AllFields[FieldIndex].Entry1;
			const FCbFieldView& Entry2Child = AllFields[FieldIndex].Entry2;
			if (Entry1Child && Entry2Child)
			{
				ReportCompactBinaryDifferences(PropertyNameStack, Entry1Child, Entry2Child, OutResults);
			}
			else
			{
				OutResults.Add(
				{
					.PropertyName = CreatePropertyHierarchicalName(PropertyNameStack, AllKeys[FieldIndex]),
					.OldValue = Entry1Child,
					.NewValue = Entry2Child
				});
			}
		}
	}

	// This is making the assumption that arrays are sorted in some fashion!
	// Testing unsorted arrays is *much* slower
	void ReportCompactBinaryDifferences(TArray<FUtf8StringView>& PropertyNameStack, FCbArrayView Entry1, FCbArrayView Entry2, TArray<FCbEntryDifference>& OutResults)
	{
		if (Entry1.Num() == Entry2.Num())
		{
			FCbFieldViewIterator Array1Iterator = Entry1.CreateViewIterator();
			FCbFieldViewIterator Array2Iterator = Entry2.CreateViewIterator();
			int32 ArrayIndex = 0;
			UTF8CHAR ArrayIndexBuffer[32] = { 0 };
			while (Array1Iterator)
			{
				FCStringUtf8::Sprintf(ArrayIndexBuffer, "%d", ArrayIndex++);
				PropertyNameStack.Push(ArrayIndexBuffer);
				ReportCompactBinaryDifferences(PropertyNameStack, Array1Iterator, Array2Iterator, OutResults);
				PropertyNameStack.Pop();
				++Array1Iterator;
				++Array2Iterator;
			}
		}
		else
		{
			// Arrays of different sizes, just report the array property
			OutResults.Add(
			{
				.PropertyName = CreatePropertyHierarchicalName(PropertyNameStack),
				.OldValue = Entry1.AsFieldView(),
				.NewValue = Entry2.AsFieldView()
			});
		}
	}

	void ReportCompactBinaryDifferences(TArray<FUtf8StringView>& PropertyNameStack, FCbFieldView Entry1, FCbFieldView Entry2, TArray<FCbEntryDifference>& OutResults)
	{
		const bool EntryHasName = Entry1.HasName() && Entry2.HasName();
		if (EntryHasName)
		{
			PropertyNameStack.Push(Entry1.GetName());
		}
		if (Entry1.IsObject() && Entry2.IsObject())
		{
			ReportCompactBinaryDifferences(PropertyNameStack, Entry1.AsObjectView(), Entry2.AsObjectView(), OutResults);
		}
		else if (Entry1.IsArray() && Entry2.IsArray())
		{
			ReportCompactBinaryDifferences(PropertyNameStack, Entry1.AsArrayView(), Entry2.AsArrayView(), OutResults);
		}
		else if (!Entry1.Equals(Entry2))
		{
			OutResults.Add(
			{
				.PropertyName = CreatePropertyHierarchicalName(PropertyNameStack),
				.OldValue = Entry1,
				.NewValue = Entry2
			});
		}
		if (EntryHasName)
		{
			PropertyNameStack.Pop();
		}
	}

	TArray<FCbEntryDifference> DiffCompactBinary(const FCbFieldView& cb1, const FCbFieldView& cb2)
	{
		TArray<FCbEntryDifference> Results;
		TArray<FUtf8StringView> PropertyNames;	// Keep a stack of property names
		PropertyNames.Reserve(10);
		ReportCompactBinaryDifferences(PropertyNames, cb1, cb2, Results);
		return Results;
	}

}	// namespace UE
