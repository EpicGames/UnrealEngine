// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/ZenOplogDiff.h"
#include "Experimental/ZenOplogManifest.h"
#include "Algo/Sort.h"
#include "Async/ParallelFor.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryValue.h"
#include "Serialization/CompactBinaryWriter.h"

DEFINE_LOG_CATEGORY_STATIC(LogZenOplogUtils, Log, All);

namespace UE
{
	bool OpDiffContainsCookedOutputChange(const UE::FOplogDiffChangedOp& Op)
	{
		static const FUtf8String PackageDataArrayName = TEXT("packagedata");
		static const FUtf8String BulkDataArrayName = TEXT("bulkdata");
		static const FUtf8String FilesArrayName = TEXT("files");
		static const FUtf8String CookedDataHashPropertyName = TEXT("data");
		return (OpDiffContainsArrayValueChange(Op, PackageDataArrayName, CookedDataHashPropertyName) ||
			OpDiffContainsArrayValueChange(Op, BulkDataArrayName, CookedDataHashPropertyName) ||
			OpDiffContainsArrayValueChange(Op, FilesArrayName, CookedDataHashPropertyName));
	}

	FOplogDiffResults DiffManifests(const FOplogManifest& Manifest1, const FOplogManifest& Manifest2)
	{
		FOplogDiffResults Results;
		Results.Manifest1Data = Manifest1.AllOpsCbData;
		Results.Manifest2Data = Manifest2.AllOpsCbData;

		// Find all ops that exist in manifest 2 but not in manifest 1
		{
			TArray<FOplogDiffResults> TaskContexts;
			ParallelForWithTaskContext(TaskContexts, Manifest2.Ops.Num(), [&Manifest1, &Manifest2](FOplogDiffResults& RunContext, int32 OpIndex) 
			{
				const FUtf8String& Op2Key = Manifest2.Ops[OpIndex].GetKey();
				const int32* FoundManifest1Index = Manifest1.OpKeyToIndex.Find(Op2Key);
				if (FoundManifest1Index == nullptr)
				{
					RunContext.OpsMissingInManifest1.Add(Op2Key);
				}
			});
			for (FOplogDiffResults& TaskResult : TaskContexts)	// Combine results
			{
				Results.OpsMissingInManifest1.Append(MoveTemp(TaskResult.OpsMissingInManifest1));
			}
		}

		// Diff all ops in manifest1 against manifest 2
		{
			TArray<FOplogDiffResults> TaskContexts;
			ParallelForWithTaskContext(TaskContexts, Manifest1.Ops.Num(), [&Manifest1, &Manifest2](FOplogDiffResults& RunContext, int32 OpIndex) 
			{
				const TPair<FUtf8String, uint32>& Op1KeyAndHash = Manifest1.OpKeysAndHashes[OpIndex];
				const int32* FoundManifest2Index = Manifest2.OpKeyToIndex.FindByHash(Op1KeyAndHash.Value, Op1KeyAndHash.Key);
				if (FoundManifest2Index == nullptr)
				{
					RunContext.OpsMissingInManifest2.Add(Op1KeyAndHash.Key);
				}
				else
				{
					const FOplogManifest::FOp& Op2 = Manifest2.Ops[*FoundManifest2Index];
					TArray<FCbEntryDifference> Differences = DiffCompactBinary(Manifest1.Ops[OpIndex].CbData, Op2.CbData);
					if (Differences.Num() == 0)
					{
						RunContext.IdenticalOps.Add(Op1KeyAndHash.Key);
					}
					else
					{
						Algo::SortBy(Differences, &FCbEntryDifference::PropertyName);	// Sort changed properties by name
						FOplogDiffChangedOp ChangedOp = { Op1KeyAndHash.Key , MoveTemp(Differences) };
						if (OpDiffContainsCookedOutputChange(ChangedOp))
						{
							RunContext.ChangedOpsWithOutputDifferences.Emplace(MoveTemp(ChangedOp));
						} 
						else
						{
							RunContext.ChangedOpsWithSameOutput.Emplace(MoveTemp(ChangedOp));
						}
					}
				}
			});
			for (FOplogDiffResults& TaskResult : TaskContexts)	// Combine results
			{
				Results.ChangedOpsWithOutputDifferences.Append(MoveTemp(TaskResult.ChangedOpsWithOutputDifferences));
				Results.ChangedOpsWithSameOutput.Append(MoveTemp(TaskResult.ChangedOpsWithSameOutput));
				Results.IdenticalOps.Append(MoveTemp(TaskResult.IdenticalOps));
				Results.OpsMissingInManifest2.Append(MoveTemp(TaskResult.OpsMissingInManifest2));
			}
		}

		return Results;
	}

	bool ChangeIsArrayOrEntryModification(const FCbEntryDifference& Entry, FUtf8StringView ArrayName, FUtf8StringView PropertyName)
	{
		check(ArrayName.Len() > 0 && PropertyName.Len() > 0);
		FUtf8StringView EntryPropName(Entry.PropertyName);
		if (EntryPropName.StartsWith(ArrayName, ESearchCase::IgnoreCase))
		{
			EntryPropName.RightChopInline(ArrayName.Len());
			if (EntryPropName.Len() == 0)
			{
				return true;	// Entry contains the array name only
			}
			if (EntryPropName.EndsWith(PropertyName, ESearchCase::IgnoreCase))
			{
				EntryPropName.LeftChopInline(PropertyName.Len());
				if (EntryPropName.StartsWith('/') && EntryPropName.EndsWith('/') && EntryPropName.Len() > 2)
				{
					EntryPropName.MidInline(1, EntryPropName.Len() - 2);
					// Ensure anything remaining parses to an integer array index
					auto RemainingAsTchar = StringCast<TCHAR>(EntryPropName.GetData(), EntryPropName.Len());
					TCHAR* TextEnd = nullptr;
					FCString::Strtoi(RemainingAsTchar.Get(), &TextEnd, 10);
					if (TextEnd == (RemainingAsTchar.Get() + RemainingAsTchar.Length()))
					{
						return true;
					}
				}
			}
		}
		return false;
	}

	bool OpDiffContainsArrayValueChange(const FOplogDiffChangedOp& Op, FUtf8StringView ArrayName, FUtf8StringView PropertyName)
	{
		for (const FCbEntryDifference& Entry : Op.Value)
		{
			if (ChangeIsArrayOrEntryModification(Entry, ArrayName, PropertyName))
			{
				return true;
			}
		}
		return false;
	}

	void SortDiffResults(FOplogDiffResults& FullResults)
	{
		Algo::Sort(FullResults.IdenticalOps);
		Algo::Sort(FullResults.OpsMissingInManifest1);
		Algo::Sort(FullResults.OpsMissingInManifest2);
		Algo::SortBy(FullResults.ChangedOpsWithSameOutput, &FOplogDiffChangedOp::Key);
		Algo::SortBy(FullResults.ChangedOpsWithOutputDifferences, &FOplogDiffChangedOp::Key);
	}

	// Helper for outputting ops
	template<typename OpEntry>
	void OutputEachOp(const TArray<OpEntry>& Results, TFunction<void(const OpEntry&)> OutputFn)
	{
		for (const OpEntry& Package : Results)
		{
			OutputFn(Package);
		}
	}

	template<bool bOutputPropertyDifferences>
	void OutputManifestDiffResultsToJsonImpl(TSharedRef<TJsonWriter<>>& JsonWriter, const FOplogDiffResults& Results, bool bOutputIdentical)
	{
		auto OutputChangedOp = [&JsonWriter](const FOplogDiffChangedOp& Package)
		{
			JsonWriter->WriteObjectStart();
			JsonWriter->WriteValue("Key", Package.Key);
			JsonWriter->WriteArrayStart("Diffs");
			TUtf8StringBuilder<1024> PropertyDataAsJson;
			for (const FCbEntryDifference& Difference : Package.Value)
			{
				JsonWriter->WriteObjectStart();
				JsonWriter->WriteValue("Property", Difference.PropertyName);
				if constexpr (bOutputPropertyDifferences)
				{
					if (Difference.OldValue.HasValue())
					{
						PropertyDataAsJson.Reset();
						CompactBinaryToJson(Difference.OldValue, PropertyDataAsJson);
						JsonWriter->WriteArrayStart("Old");	// We must surround values in arrays since objects cannot contain un-named values
						JsonWriter->WriteRawJSONValue(PropertyDataAsJson.ToString());
						JsonWriter->WriteArrayEnd();
					}
					if (Difference.NewValue.HasValue())
					{
						PropertyDataAsJson.Reset();
						CompactBinaryToJson(Difference.NewValue, PropertyDataAsJson);
						JsonWriter->WriteArrayStart("New");
						JsonWriter->WriteRawJSONValue(PropertyDataAsJson.ToString());
						JsonWriter->WriteArrayEnd();
					}
				}
				JsonWriter->WriteObjectEnd();
			}
			JsonWriter->WriteArrayEnd();	// Diffs
			JsonWriter->WriteObjectEnd();
		};
		JsonWriter->WriteObjectStart("Diff");
		JsonWriter->WriteArrayStart("ChangedOpsWithSameOutput");
		OutputEachOp<FOplogDiffChangedOp>(Results.ChangedOpsWithSameOutput, OutputChangedOp);
		JsonWriter->WriteArrayEnd();	// ChangedOpsSameOutput
		JsonWriter->WriteArrayStart("ChangedOpsWithOutputDifferences");
		OutputEachOp<FOplogDiffChangedOp>(Results.ChangedOpsWithOutputDifferences, OutputChangedOp);
		JsonWriter->WriteArrayEnd();	// ChangedOpsWithOutputDifferences
		JsonWriter->WriteArrayStart("MissingInManifest1");
		OutputEachOp<FUtf8String>(Results.OpsMissingInManifest1, [&JsonWriter](const FUtf8String& Package)
		{
			JsonWriter->WriteValue(Package);
		});
		JsonWriter->WriteArrayEnd();	// MissingInManifest1
		JsonWriter->WriteArrayStart("MissingInManifest2");
		OutputEachOp<FUtf8String>(Results.OpsMissingInManifest2, [&JsonWriter](const FUtf8String& Package)
		{
			JsonWriter->WriteValue(Package);
		});
		JsonWriter->WriteArrayEnd();	// MissingInManifest2
		if (bOutputIdentical)
		{
			JsonWriter->WriteArrayStart("Identical");
			OutputEachOp<FUtf8String>(Results.IdenticalOps, [&JsonWriter](const FUtf8String& Package)
			{
				JsonWriter->WriteValue(Package);
			});
			JsonWriter->WriteArrayEnd();	// Identical
		}
		JsonWriter->WriteObjectEnd();
	}

	template<bool bOutputPropertyDifferences>
	void OutputManifestDiffResultsToCompactBinaryImpl(FCbWriter& Writer, const FOplogDiffResults& Results, bool bOutputIdentical)
	{
		auto OutputChangedOp = [&Writer](const FOplogDiffChangedOp& Package)
		{
			Writer.BeginObject();
			Writer.AddString("Key", Package.Key);
			Writer.BeginArray("Diffs");
			for (const FCbEntryDifference& Difference : Package.Value)
			{
				Writer.BeginObject();
				Writer.AddString("Property", Difference.PropertyName);
				if constexpr (bOutputPropertyDifferences)
				{
					if (Difference.OldValue.HasValue())
					{
						Writer.BeginArray("Old");
						Writer.AddField(Difference.OldValue);
						Writer.EndArray();
					}
					if (Difference.NewValue.HasValue())
					{
						Writer.BeginArray("New");
						Writer.AddField(Difference.NewValue);
						Writer.EndArray();
					}
				}
				Writer.EndObject();
			}
			Writer.EndArray();	// diffs
			Writer.EndObject();
		};
		Writer.BeginObject("Diff");
		Writer.BeginArray("ChangedOpsWithSameOutput");
		OutputEachOp<FOplogDiffChangedOp>(Results.ChangedOpsWithSameOutput, OutputChangedOp);
		Writer.EndArray();	// ChangedOpsSameOutput
		Writer.BeginArray("ChangedOpsWithOutputDifferences");
		OutputEachOp<FOplogDiffChangedOp>(Results.ChangedOpsWithOutputDifferences, OutputChangedOp);
		Writer.EndArray();	// ChangedOpsWithOutputDifferences
		Writer.BeginArray("MissingInManifest1");
		OutputEachOp<FUtf8String>(Results.OpsMissingInManifest1, [&Writer](const FUtf8String& Package)
		{
			Writer.AddString(Package);
		});
		Writer.EndArray();	// MissingInManifest1
		Writer.BeginArray("MissingInManifest2");
		OutputEachOp<FUtf8String>(Results.OpsMissingInManifest2, [&Writer](const FUtf8String& Package)
		{
			Writer.AddString(Package);
		});
		Writer.EndArray();	// MissingInManifest2
		if (bOutputIdentical)
		{
			Writer.BeginArray("Identical");
			OutputEachOp<FUtf8String>(Results.IdenticalOps, [&Writer](const FUtf8String& Package)
			{
				Writer.AddString(Package);
			});
			Writer.EndArray();	// Identical
		}
		
		Writer.EndObject();
	}

	void OutputManifestDiffResultsToJson(TSharedRef<TJsonWriter<>>& Writer, const FOplogDiffResults& Results, EOutputManifestDiffOptions Flags)
	{
		const bool bOutputIdentical = EnumHasAnyFlags(Flags, EOutputManifestDiffOptions::OutputIdenticalOps);
		if (EnumHasAnyFlags(Flags, EOutputManifestDiffOptions::OutputDifferences))
		{
			OutputManifestDiffResultsToJsonImpl<true>(Writer, Results, bOutputIdentical);
		}
		else
		{
			OutputManifestDiffResultsToJsonImpl<false>(Writer, Results, bOutputIdentical);
		}
	}

	void OutputManifestDiffResultsToCompactBinary(FCbWriter& Writer, const FOplogDiffResults& Results, EOutputManifestDiffOptions Flags)
	{
		const bool bOutputIdentical = EnumHasAnyFlags(Flags, EOutputManifestDiffOptions::OutputIdenticalOps);
		if (EnumHasAnyFlags(Flags, EOutputManifestDiffOptions::OutputDifferences))
		{
			return OutputManifestDiffResultsToCompactBinaryImpl<true>(Writer, Results, bOutputIdentical);
		}
		else
		{
			return OutputManifestDiffResultsToCompactBinaryImpl<false>(Writer, Results, bOutputIdentical);
		}
	}

}	// namespace UE
