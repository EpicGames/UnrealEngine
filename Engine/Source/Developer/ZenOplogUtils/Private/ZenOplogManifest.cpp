// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/ZenOplogManifest.h"
#include "Experimental/DiffCompactBinary.h"
#include "Async/ParallelFor.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"

namespace UE
{

	FUtf8String FOplogManifest::FOp::GetKey() const
	{
		return FUtf8String(CbData["key"].AsString());
	}

	FIoHash FOplogManifest::FOp::GetArtifactHash() const
	{
		return CbData["meta.cook.artifacts"].AsBinaryAttachment();
	}

	FIoHash FOplogManifest::FOp::CookImportExport() const
	{
		return CbData["meta.cook.importexport"].AsBinaryAttachment();
	}

	FIoHash FOplogManifest::FOp::GetLogs() const
	{
		return CbData["meta.cook.logs"].AsBinaryAttachment();
	}

	FOplogManifest::FOp::FPackageStoreEntry FOplogManifest::FOp::GetPackageStoreEntry() const
	{
		FOplogManifest::FOp::FPackageStoreEntry Data;
		if (FCbObjectView PackageStoreEntry = CbData["packagestoreentry"].AsObjectView())
		{
			Data.Flags = PackageStoreEntry["flags"].AsUInt32();
			Data.PackageName = PackageStoreEntry["packagename"].AsString();
			if (PackageStoreEntry["importedpackageids"])
			{
				for (FCbFieldView ArrayField : PackageStoreEntry["importedpackageids"])
				{
					Data.ImportedPackages.Add(ArrayField.AsUInt64());
				}
			}
			if (PackageStoreEntry["softpackagereferences"])
			{
				for (FCbFieldView ArrayField : PackageStoreEntry["softpackagereferences"])
				{
					Data.SoftPackageReferences.Add(ArrayField.AsUInt64());
				}
			}
		}
		return Data;
	}

	TArray<FOplogManifest::FOp::FPackageData> FOplogManifest::FOp::GetPackageDatas() const
	{
		TArray<FOplogManifest::FOp::FPackageData> ParsedData;
		FCbArrayView PackageDataArray = CbData["packagedata"].AsArrayView();
		for (FCbFieldView PackageDataEntry : PackageDataArray)
		{
			FCbObjectView PackageDataObject = PackageDataEntry.AsObjectView();
			FOplogManifest::FOp::FPackageData NewPackageData;
			NewPackageData.ID = PackageDataEntry["id"].AsObjectId();
			NewPackageData.Size = PackageDataEntry["size"].AsInt64();
			NewPackageData.RawSize = PackageDataEntry["rawsize"].AsInt64();
			NewPackageData.DataHash = PackageDataEntry["data"].AsBinaryAttachment();
			NewPackageData.Filename = PackageDataEntry["filename"].AsString();
			ParsedData.Add(NewPackageData);
		}
		return ParsedData;
	}

	TArray<FOplogManifest::FOp::FBulkData> FOplogManifest::FOp::GetBulkDatas() const
	{
		TArray<FOplogManifest::FOp::FBulkData> ParsedData;
		FCbArrayView BulkDataArray = CbData["bulkdata"].AsArrayView();
		for (FCbFieldView BulkDataEntry : BulkDataArray)
		{
			FCbObjectView BulkDataObject = BulkDataEntry.AsObjectView();
			FOplogManifest::FOp::FBulkData NewBulkData;
			NewBulkData.ID = BulkDataEntry["id"].AsObjectId();
			NewBulkData.TypeStr = BulkDataEntry["type"].AsString();
			NewBulkData.Size = BulkDataEntry["size"].AsInt64();
			NewBulkData.RawSize = BulkDataEntry["rawsize"].AsInt64();
			NewBulkData.DataHash = BulkDataEntry["data"].AsBinaryAttachment();
			NewBulkData.Filename = BulkDataEntry["filename"].AsString();
			ParsedData.Add(NewBulkData);
		}
		return ParsedData;
	}

	TArray<FOplogManifest::FOp::FFile> FOplogManifest::FOp::GetFiles() const
	{
		TArray<FOplogManifest::FOp::FFile> ParsedData;
		FCbArrayView FilesArray = CbData["files"].AsArrayView();
		for (FCbFieldView FileEntry : FilesArray)
		{
			FOplogManifest::FOp::FFile NewFile;
			NewFile.ID = FileEntry["id"].AsObjectId();
			NewFile.ServerPath = FileEntry["serverpath"].AsString();
			NewFile.ClientPath = FileEntry["clientpath"].AsString();
			NewFile.DataHash = FileEntry["data"].AsBinaryAttachment();
			ParsedData.Add(NewFile);
		}
		return ParsedData;
	}

	TArray<FOplogManifest::FOp::FMeta> FOplogManifest::FOp::GetMetas() const
	{
		TArray<FOplogManifest::FOp::FMeta> ParsedData;
		FCbArrayView MetaArray = CbData["meta"].AsArrayView();
		for (FCbFieldView MetaEntry : MetaArray)
		{
			FOplogManifest::FOp::FMeta NewMeta;
			NewMeta.ID = MetaEntry["id"].AsObjectId();
			NewMeta.Name = MetaEntry["name"].AsString();
			NewMeta.DataHash = MetaEntry["data"].AsBinaryAttachment();
			ParsedData.Add(NewMeta);
		}
		return ParsedData;
	}

	bool FOplogManifest::RebuildCachedData()
	{
		check(OpKeyToIndex.Num() == 0);
		check(OpKeysAndHashes.Num() == 0);
		OpKeyToIndex.Reserve(Ops.Num());
		OpKeysAndHashes.AddDefaulted(Ops.Num());

		// Collect the key hashes and key -> index mappings on task threads
		struct CollectedOpKeyToIndex
		{
			FUtf8String Key;
			uint32 KeyHash;
			int32 OpIndex;
		};
		TArray<TArray<CollectedOpKeyToIndex>> PerTaskOpKeyToIndex;
		ParallelForWithTaskContext(PerTaskOpKeyToIndex, Ops.Num(), [this](TArray<CollectedOpKeyToIndex>& ThisTaskOpKeyToIndex, int32 OpIndex) 
		{
			FUtf8String ThisKey = Ops[OpIndex].GetKey();
			uint32 ThisKeyHash = GetTypeHash(ThisKey);
			ThisTaskOpKeyToIndex.Emplace(ThisKey, ThisKeyHash, OpIndex);
			OpKeysAndHashes[OpIndex] = { MoveTemp(ThisKey), ThisKeyHash };
		});

		// Combine results
		bool bDuplicateKeyFound = false;
		for (TArray<CollectedOpKeyToIndex>& PerTaskEntries : PerTaskOpKeyToIndex)
		{
			for (CollectedOpKeyToIndex& Entry : PerTaskEntries)
			{
				bDuplicateKeyFound |= (OpKeyToIndex.FindOrAddByHash(Entry.KeyHash, MoveTemp(Entry.Key), Entry.OpIndex) != Entry.OpIndex);
			}
		}
		return !bDuplicateKeyFound;
	}

	FLoadOplogManifestResult::FLoadOplogManifestResult(FLoadOplogManifestResult::EStatus Stat, TOptional<FOplogManifest>&& Man)
		: Result(Stat)
		, Manifest(MoveTemp(Man))
	{
	}

	FLoadOplogManifestResult LoadOplogManifestFromCompactBinary(const FCbObject& CompactBinary)
	{
		FOplogManifest LoadedManifest;

		FCbField AllOps = CompactBinary["ops"];
		if (!AllOps.IsArray())
		{
			return FLoadOplogManifestResult(FLoadOplogManifestResult::EStatus::ErrorMalformedCompactBinary);
		}

		LoadedManifest.AllOpsCbData = MoveTemp(AllOps);
		for (FCbFieldView OpEntry : LoadedManifest.AllOpsCbData)
		{
			if (!OpEntry["key"].IsString())	// We expect each op entry to contain a key at minimum
			{
				return FLoadOplogManifestResult(FLoadOplogManifestResult::EStatus::ErrorMalformedCompactBinary);
			}

			FOplogManifest::FOp NewOpData;
			NewOpData.CbData = MoveTemp(OpEntry);
			LoadedManifest.Ops.Add(MoveTemp(NewOpData));
		}
		if (!LoadedManifest.RebuildCachedData())
		{
			return FLoadOplogManifestResult(FLoadOplogManifestResult::EStatus::Error);
		}
		return FLoadOplogManifestResult(FLoadOplogManifestResult::EStatus::Ok, MoveTemp(LoadedManifest));
	}

	FLoadOplogManifestResult LoadOplogManifestFromFile(FString Path)
	{
		FString FileExtension = FPaths::GetExtension(Path).ToLower();
		if (FileExtension == TEXT("cb"))
		{
			TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*Path));
			if (Ar)
			{
				FCbObject RootObject = LoadCompactBinary(*Ar).AsObject();
				if (!RootObject)
				{
					return FLoadOplogManifestResult(FLoadOplogManifestResult::EStatus::ErrorMalformedCompactBinary);
				}
				return LoadOplogManifestFromCompactBinary(RootObject);
			}
			else
			{
				return FLoadOplogManifestResult(FLoadOplogManifestResult::EStatus::ErrorFileNotFound);
			}
		}
		return FLoadOplogManifestResult(FLoadOplogManifestResult::EStatus::ErrorUnsupportedFormat);
	}

}	// namespace UE
