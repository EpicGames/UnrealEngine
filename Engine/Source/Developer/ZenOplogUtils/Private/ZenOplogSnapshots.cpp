// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/ZenOplogSnapshots.h"
#include "Experimental/BuildServerInterface.h"
#include "Experimental/ZenServerInterface.h"
#include "DesktopPlatformModule.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonSerializer.h"

DEFINE_LOG_CATEGORY_STATIC(LogOplogSnapshotUtils, Log, All);

namespace UE
{
	int32 RunZenExeSync(const TCHAR* CommandLine)
	{
		int32 ReturnCode = -1;
		FString ZenUtilPath = FPaths::ConvertRelativePathToFull(UE::Zen::GetLocalInstallUtilityPath());

		FProcHandle ProcHandle = FPlatformProcess::CreateProc(*ZenUtilPath, CommandLine, false, false, false, nullptr, 0, nullptr, nullptr);
		if (ProcHandle.IsValid())
		{
			FPlatformProcess::WaitForProc(ProcHandle);
			if (!FPlatformProcess::GetProcReturnCode(ProcHandle, &ReturnCode))
			{
				UE_LOGF(LogOplogSnapshotUtils, Warning, "Failed to get return code from zen process");
			}
			FPlatformProcess::CloseProc(ProcHandle);
		}
		else
		{
			UE_LOGF(LogOplogSnapshotUtils, Warning, "Failed to launch zen utility from %ls", *ZenUtilPath);
		}
		return ReturnCode;
	}

	FDownloadOplogManifestResult DownloadOplogManifest(const FZenSnapshotDescriptor& Descriptor, FString OutputDirectory, bool bDownloadAsJson)
	{
		FString OidcTokenExePath = FPaths::ConvertRelativePathToFull(FDesktopPlatformModule::Get()->GetOidcTokenExecutableFilename(FPaths::RootDir()));
		FString OutputDir = FPaths::ConvertRelativePathToFull(FPaths::Combine(FDesktopPlatformModule::Get()->GetUserTempPath(), TEXT("OplogManifests")));
		FString ManifestFileType = bDownloadAsJson ? TEXT("json") : TEXT("cb");
		if (!OutputDirectory.IsEmpty() && FPaths::DirectoryExists(OutputDirectory))
		{
			OutputDir = OutputDirectory;
		}

		FString ManifestPath = FPaths::Combine(OutputDir, FPaths::MakeValidFileName(FString::Printf(TEXT("%s-%s-%s.%s"), *Descriptor.Namespace, *Descriptor.Bucket, *Descriptor.BuildID, *ManifestFileType)));
		
		TStringBuilder<1024> CommandLineArgs;
		CommandLineArgs.Append(TEXT("oplog-download"));
		CommandLineArgs.Append(FString::Printf(TEXT(" --host \"%s\""), *Descriptor.CloudHost));
		CommandLineArgs.Append(FString::Printf(TEXT(" --namespace \"%s\""), *Descriptor.Namespace));
		CommandLineArgs.Append(FString::Printf(TEXT(" --bucket \"%s\""), *Descriptor.Bucket));
		CommandLineArgs.Append(FString::Printf(TEXT(" --build-id \"%s\""), *Descriptor.BuildID));
		CommandLineArgs.Append(FString::Printf(TEXT(" --oidctoken-exe-path \"%s\""), *OidcTokenExePath));
		CommandLineArgs.Append(FString::Printf(TEXT(" --output-path \"%s\""), *ManifestPath));
		if (FApp::IsUnattended())
		{
			CommandLineArgs.Append(TEXT(" --oidctoken-exe-unattended"));
		}

		int32 ZenResult = RunZenExeSync(*CommandLineArgs);
		return FDownloadOplogManifestResult
		{
			.Result = (ZenResult == 0) ? FDownloadOplogManifestResult::EStatus::Ok : FDownloadOplogManifestResult::EStatus::Error,
			.DownloadedFilename = (ZenResult == 0) ? FString(ManifestPath) : FString()
		};
	}

	TArray<FZenSnapshotDescriptor> ParseSnapshotDescriptorFromJson(FStringView PathToJson)
	{
		TArray<FZenSnapshotDescriptor> NewSnapshots;
		FString JsonFileContent;
		if (FFileHelper::LoadFileToString(JsonFileContent, *WriteToString<256>(PathToJson)))
		{
			TSharedPtr<FJsonObject> TopLevelObject;
			const TSharedRef<TJsonReader<>>& Reader = TJsonReaderFactory<>::Create(JsonFileContent);
			if (FJsonSerializer::Deserialize(Reader, TopLevelObject))
			{
				if (const TSharedPtr <FJsonValue> SnapshotValues = TopLevelObject->GetFieldUntyped(TEXT("snapshots")); SnapshotValues.IsValid())
				{
					for (const TSharedPtr<FJsonValue>& SnapshotEntry : SnapshotValues->AsArray())
					{
						if (const TSharedPtr<FJsonObject> SnapshotJson = SnapshotEntry->AsObject())
						{
							NewSnapshots.Add(
							{
								.CloudHost = SnapshotJson->GetStringField(TEXT("host")),
								.Namespace = SnapshotJson->GetStringField(TEXT("namespace")),
								.Bucket = SnapshotJson->GetStringField(TEXT("bucket")),
								.BuildID = SnapshotJson->GetStringField(TEXT("builds-id"))
							});
						}
					}
				}
			}
			else
			{
				UE_LOGF(LogOplogSnapshotUtils, Warning, "Failed to parse json in %ls", *FString(PathToJson));
			}
		}
		else
		{
			UE_LOGF(LogOplogSnapshotUtils, Warning, "Failed to load snapshot file %ls", *FString(PathToJson));
		}
		return NewSnapshots;
	}

	// Data used during the async bits of FindSnapshotDescriptorForBuild
	struct FFindSnapshotDescriptorContext
	{
		// The project/stream/platform/cl/build to search for
		FString FindProjectName;
		FString FindStreamName;
		FString FindPlatformName;
		FString FindBuildType;
		FString FindCommitID;

		bool bLogAllOutput;		// Whether to log all build info found

		// The hostname/namespace/bucket found during FindSnapshotDescriptorForBuild
		FString FoundHostname;
		FString FoundNamespace;
		FString FoundBucket;

		TSharedPtr<UE::Zen::Build::FBuildServiceInstance> BuildService;
		TPromise<TOptional<FZenSnapshotDescriptor>> ResultPromise;
	};

	// Called when the build service has listed a number of builds matching the search in FFindSnapshotDescriptorContext
	void OnBuildsFound(TSharedPtr<FFindSnapshotDescriptorContext> Context, TArray<UE::Zen::Build::FBuildServiceInstance::FBuildRecord>&& Builds)
	{
		for (const UE::Zen::Build::FBuildServiceInstance::FBuildRecord& BuildRecord : Builds)
		{
			FString CommitID = BuildRecord.GetCommitIdentifier();
			if (Context->bLogAllOutput)
			{
				UE_LOGF(LogOplogSnapshotUtils, Display, "\t%ls", *CommitID);
			}
			if (CommitID == Context->FindCommitID)
			{
				FZenSnapshotDescriptor FoundSnapshot;
				FoundSnapshot.Namespace = Context->FoundNamespace;
				FoundSnapshot.CloudHost = Context->FoundHostname;
				FoundSnapshot.BuildID = WriteToString<64>(BuildRecord.BuildId);
				FoundSnapshot.Bucket = Context->FoundBucket;
				Context->ResultPromise.SetValue(FoundSnapshot);
				return;
			}
		}
		UE_LOGF(LogOplogSnapshotUtils, Warning, "Failed to find a matching build with commit ID %ls in %ls/%ls", *Context->FindCommitID, *Context->FoundNamespace, *Context->FoundBucket);
		Context->ResultPromise.SetValue({});
	}

	// Called when the build service has refreshed the list of all available namespaces/buckets
	void OnNamespacesAndBucketsRefreshed(TSharedPtr<FFindSnapshotDescriptorContext> Context)
	{
		TMultiMap<FString, FString> NamespacesAndBuckets = Context->BuildService->GetNamespacesAndBuckets();
		TStringBuilder<256> BucketNameToFind;	// bucket names are expected in the form of: projectname.buildtype.stream.platform
		BucketNameToFind << Context->FindProjectName.ToLower() << TEXT(".");
		BucketNameToFind << Context->FindBuildType.ToLower() << TEXT(".");
		BucketNameToFind << Context->FindStreamName.ToLower() << TEXT(".");
		BucketNameToFind << Context->FindPlatformName.ToLower();

		if (Context->bLogAllOutput)
		{
			UE_LOGF(LogOplogSnapshotUtils, Display, "Searching for bucket %ls", *BucketNameToFind);
		}

		for (const TPair<FString, FString>& NamespaceAndBucket : NamespacesAndBuckets)
		{
			if (Context->bLogAllOutput)
			{
				UE_LOGF(LogOplogSnapshotUtils, Display, "%ls -> %ls", *NamespaceAndBucket.Key, *NamespaceAndBucket.Value);
			}
			if (NamespaceAndBucket.Value == BucketNameToFind.ToString())
			{
				// Now query for builds in the bucket
				Context->FoundNamespace = NamespaceAndBucket.Key;
				Context->FoundBucket = NamespaceAndBucket.Value;
				Context->BuildService->ListBuilds(NamespaceAndBucket.Key, NamespaceAndBucket.Value, FCbObject{},
					[Context](TArray<UE::Zen::Build::FBuildServiceInstance::FBuildRecord>&& BuildRecords)
				{
					OnBuildsFound(Context, MoveTemp(BuildRecords));
				});
				return;
			}
		}

		UE_LOGF(LogOplogSnapshotUtils, Warning, "Failed to find a matching build namespace/bucket");
		Context->ResultPromise.SetValue({});
	}

	// Called when the build service successfully connects
	void OnBuildServiceConnected(TSharedPtr<FFindSnapshotDescriptorContext> Context)
	{
		Context->FoundHostname = FString(Context->BuildService->GetEffectiveDomain());
		Context->BuildService->RefreshNamespacesAndBuckets([Context]()
		{
			ExecuteOnGameThread(UE_SOURCE_LOCATION, [Context]()
			{
				OnNamespacesAndBucketsRefreshed(Context);
			});
		});
	}

	TFuture<TOptional<FZenSnapshotDescriptor>> FindSnapshotDescriptorForBuild(FStringView ProjectName, FStringView StreamName, FStringView PlatformName, FStringView CommitID, bool bLogAllBuilds)
	{
		using namespace UE::Zen::Build;

		TSharedPtr<FFindSnapshotDescriptorContext> Context = MakeShared<FFindSnapshotDescriptorContext>();
		Context->BuildService = MakeShared<FBuildServiceInstance>();
		Context->FindProjectName = FString(ProjectName).ToLower();
		Context->FindStreamName = FString(StreamName).ToLower();
		Context->FindPlatformName = FString(PlatformName).ToLower();
		Context->FindCommitID = CommitID;
		Context->FindBuildType = "oplog";
		Context->bLogAllOutput = bLogAllBuilds;

		TFuture<TOptional<FZenSnapshotDescriptor>> ResultFuture = Context->ResultPromise.GetFuture();
		Context->BuildService->Connect(!FApp::IsUnattended(), [Context]
		(FBuildServiceInstance::EConnectionState ConnectionState,
			FBuildServiceInstance::EConnectionFailureReason FailureReason)
		{
			if (ConnectionState == UE::Zen::Build::FBuildServiceInstance::EConnectionState::ConnectionSucceeded)
			{
				OnBuildServiceConnected(Context);
			}
			else
			{
				UE_LOGF(LogOplogSnapshotUtils, Error, "Failed to connect to build service");
				switch (FailureReason)
				{
				case FBuildServiceInstance::EConnectionFailureReason::FailedToAcquireAccessToken:
					UE_LOGF(LogOplogSnapshotUtils, Error, "\tFailed to acquire access token");
					break;
				case FBuildServiceInstance::EConnectionFailureReason::FailedToResolveHost:
					UE_LOGF(LogOplogSnapshotUtils, Error, "Failed to resolve host");
					break;
				case FBuildServiceInstance::EConnectionFailureReason::UnexpectedState:
					UE_LOGF(LogOplogSnapshotUtils, Error, "Unexpected state");
					break;
				default:
					UE_LOGF(LogOplogSnapshotUtils, Error, "Unknown error");
					break;
				}
				Context->ResultPromise.SetValue({});
			}
		});
		return ResultFuture;
	}
}