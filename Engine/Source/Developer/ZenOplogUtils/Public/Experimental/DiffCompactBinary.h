// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Utf8String.h"
#include "Serialization/CompactBinary.h"

#define UE_API ZENOPLOGUTILS_API

namespace UE
{

	// This represents a single value difference in compact binary data
	//	Differences are reported with hierarchical names, so they can be found/reconstructed from the compact binary
	//		For example, compact binary containing an object named packagedata with differing filename values will return a PropertyName of 'packagedata/filename'
	//	Arrays of different sizes are reported as a single diff entry with the name of the array
	//	Arrays of equal size with different elements will have each element diff reported separately, with the index appended to the array property name
	//		e.g. 'packagestoreentry/importedpackages/2' or 'files/1/serverpath'
	struct FCbEntryDifference
	{
		FUtf8String PropertyName;
		FCbFieldView OldValue;
		FCbFieldView NewValue;
	};

	// Recursively diff the contents of a compact binary field/object
	// Important! Arrays are assumed to be sorted!
	UE_API TArray<FCbEntryDifference> DiffCompactBinary(const FCbFieldView& cb1, const FCbFieldView& cb2);

}	// namespace UE

#undef UE_API
