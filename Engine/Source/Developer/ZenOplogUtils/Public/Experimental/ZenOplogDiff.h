// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/Tuple.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/JsonWriter.h"
#include "Misc/EnumClassFlags.h"
#include "Experimental/DiffCompactBinary.h"

#define UE_API ZENOPLOGUTILS_API

namespace UE
{
	struct FOplogManifest;
	struct FCbEntryDifference;

	// Represents a changed Op and the properties with different values
	using FOplogDiffChangedOp = TPair<FUtf8String, TArray<FCbEntryDifference>>;

	// Represents the full diff of an oplog/manifest
	// Takes shared ownership of the compact binary in a manifest
	struct FOplogDiffResults
	{
		TArray<FUtf8String> OpsMissingInManifest1;
		TArray<FUtf8String> OpsMissingInManifest2;
		TArray<FUtf8String> IdenticalOps;
		TArray<FOplogDiffChangedOp> ChangedOpsWithSameOutput;		// Op has differences, but the output data is the same
		TArray<FOplogDiffChangedOp> ChangedOpsWithOutputDifferences;	// Op has cooked output differences

		FCbField Manifest1Data;
		FCbField Manifest2Data;
	};

	// Diff all ops, reports all changes in each op. Results are unsorted
	UE_API FOplogDiffResults DiffManifests(const FOplogManifest& Manifest1, const FOplogManifest& Manifest2);

	// Sorts the results of the diff
	UE_API void SortDiffResults(FOplogDiffResults& FullResults);

	// Returns true if any diff entry describes an array modification (array sizes different),
	// or if an entry in that array contains a change of a specific property
	// i.e. if changed property name == 'ArrayName' or 'ArrayName/.../PropertyName'
	UE_API bool OpDiffContainsArrayValueChange(const FOplogDiffChangedOp& Op, FUtf8StringView ArrayName, FUtf8StringView PropertyName);

	// Output diff results
	
	enum class EOutputManifestDiffOptions : int32
	{
		None = 0,
		OutputDifferences = 1,		// Output the values of any different properties
		OutputIdenticalOps = 2		// Output the list of identical ops
	};
	ENUM_CLASS_FLAGS(EOutputManifestDiffOptions);

	// Flags = a combination of EOutputManifestDiffOptions
	UE_API void OutputManifestDiffResultsToJson(TSharedRef<TJsonWriter<>>& Writer, const FOplogDiffResults& Results, EOutputManifestDiffOptions Flags );

	// Flags = a combination of EOutputManifestDiffOptions
	UE_API void OutputManifestDiffResultsToCompactBinary(FCbWriter& Writer, const FOplogDiffResults& Results, EOutputManifestDiffOptions Flags);

}	// namespace UE

#undef UE_API
