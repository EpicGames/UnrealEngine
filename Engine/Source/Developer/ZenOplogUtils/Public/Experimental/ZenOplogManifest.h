// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Utf8String.h"
#include "IO/IoHash.h"
#include "Serialization/CompactBinary.h"
#include "Templates/Tuple.h"

#define UE_API ZENOPLOGUTILS_API

namespace UE
{

	// A manifest is a chunk of compact binary representing the cooked output of a build.
	struct FOplogManifest
	{
		// Caches a view of an op in the compact binary and provides accessors to known values
		struct FOp
		{
			UE_API FUtf8String GetKey() const;
			UE_API FIoHash GetArtifactHash() const;
			UE_API FIoHash CookImportExport() const;
			UE_API FIoHash GetLogs() const;

			struct FPackageStoreEntry
			{
				uint32 Flags = 0;
				FUtf8String PackageName;
				TArray<uint64> ImportedPackages;
				TArray<uint64> SoftPackageReferences;
			};
			UE_API FPackageStoreEntry GetPackageStoreEntry() const;

			struct FPackageData
			{
				FCbObjectId ID;
				int64 Size = 0;
				int64 RawSize = 0;
				FIoHash DataHash;
				FUtf8String Filename;
			};
			UE_API TArray<FPackageData> GetPackageDatas() const;

			struct FBulkData
			{
				FCbObjectId ID;
				FUtf8String TypeStr;
				int64 Size = 0;
				int64 RawSize = 0;
				FIoHash DataHash;
				FUtf8String Filename;
			};
			UE_API TArray<FBulkData> GetBulkDatas() const;

			struct FFile
			{
				FCbObjectId ID;
				FUtf8String ServerPath;
				FUtf8String ClientPath;
				FIoHash DataHash;
			};
			UE_API TArray<FFile> GetFiles() const;

			struct FMeta
			{
				FCbObjectId ID;
				FUtf8String Name;
				FIoHash DataHash;
			};
			UE_API TArray<FMeta> GetMetas() const;

			FCbFieldView CbData;	// Cache a field view to the op data in the manifest compact binary
		};

		// Caches op keys + hashes, and populates OpKeyToIndex. Call after loading. Returns false on unexpected data
		UE_API bool RebuildCachedData();

		FCbField AllOpsCbData;
		TArray<FOp> Ops;
		TArray<TPair<FUtf8String, uint32>> OpKeysAndHashes;
		TMap<FUtf8String, int32> OpKeyToIndex;	// Op key -> index into Ops array
	};

	struct FLoadOplogManifestResult
	{
		enum class EStatus
		{
			Ok,
			ErrorFileNotFound,
			ErrorUnsupportedFormat,
			ErrorMalformedCompactBinary,
			Error
		};
		FLoadOplogManifestResult(EStatus Status, TOptional<FOplogManifest>&& Manifest = {});

		EStatus Result;
		TOptional<FOplogManifest> Manifest;
	};

	UE_API FLoadOplogManifestResult LoadOplogManifestFromCompactBinary(const FCbObject& CompactBinary);
	UE_API FLoadOplogManifestResult LoadOplogManifestFromFile(FString Path);

}	// namespace UE

#undef UE_API
