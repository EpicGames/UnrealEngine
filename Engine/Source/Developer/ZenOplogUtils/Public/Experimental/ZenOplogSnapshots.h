// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Async/Future.h"
#include "Containers/UnrealString.h"

#define UE_API ZENOPLOGUTILS_API

namespace UE
{
	// Contains the data required to get a oplog/snapshot from the cloud servers
	// Can be parsed from exported zen snapshot json, or found from the build service
	struct FZenSnapshotDescriptor
	{
		FString CloudHost;
		FString Namespace;
		FString Bucket;
		FString BuildID;
	};

	// Attempt to parse an exported zen snapshot json from horde
	UE_API TArray<FZenSnapshotDescriptor> ParseSnapshotDescriptorFromJson(FStringView PathToJson);

	// Try to find a snapshot descriptor matching a build from the zen build service
	// Returns an empty TOptional if the build was not found, or any errors occur
	// Note this relies on callbacks being fired on the game thread - do not do a blocking wait/get!
	//	bLogAllBuilds - logs all namespaces, buckets, and CLs found
	UE_API TFuture<TOptional<FZenSnapshotDescriptor>> FindSnapshotDescriptorForBuild(FStringView ProjectName, FStringView StreamName, FStringView PlatformName, FStringView CommitID, bool bLogAllBuilds = false);


	// Snapshot downloads.

	struct FDownloadOplogManifestResult
	{
		enum class EStatus
		{
			Ok,
			Error
		};
		EStatus Result;
		FString DownloadedFilename;		// Path to the local manifest file after a successful download
	};

	// Download an oplog manifest to a local file.
	UE_API FDownloadOplogManifestResult DownloadOplogManifest(const FZenSnapshotDescriptor& Descriptor, FString OutputDirectory = {}, bool bDownloadAsJson = false);
}

#undef UE_API