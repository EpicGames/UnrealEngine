// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "TestHarness.h"
#include "Experimental/DiffCompactBinary.h"
#include "Experimental/ZenOplogDiff.h"
#include "Experimental/ZenOplogManifest.h"
#include "Algo/IsSorted.h"
#include "Algo/Transform.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"

#if WITH_EDITORONLY_DATA

namespace UE
{
	namespace TestData	// Helpers for building op compact binary in the formats we expect
	{
		void WriteOp(FCbWriter& Writer, FString Key, FIoHash ArtifactHash = {}, FIoHash ImportExportHash = {}, FIoHash LogsHash = {})
		{
			Writer.AddString("key", Key);
			if (!ArtifactHash.IsZero())
			{
				Writer.AddBinaryAttachment("meta.cook.artifacts", ArtifactHash);
			}
			if (!ImportExportHash.IsZero())
			{
				Writer.AddBinaryAttachment("meta.cook.importexport", ImportExportHash);
			}
			if (!LogsHash.IsZero())
			{
				Writer.AddBinaryAttachment("meta.cook.logs", LogsHash);
			}
		}

		void WritePackageStoreEntry(FCbWriter& Writer, FString PackageName, int32 Flags, TArray<int64> ImportedPackages = {}, TArray<int64> SoftReferences = {})
		{
			Writer.BeginObject("packagestoreentry");
			Writer.AddInteger("flags", Flags);
			Writer.AddString("packagename", PackageName);
			if (ImportedPackages.Num() > 0)
			{
				Writer.BeginArray("importedpackageids");
				for (uint64 ID : ImportedPackages)
				{
					Writer.AddInteger(ID);
				}
				Writer.EndArray();	// importedpackageids
			}
			if (SoftReferences.Num() > 0)
			{
				Writer.BeginArray("softpackagereferences");
				for (uint64 ID : SoftReferences)
				{
					Writer.AddInteger(ID);
				}
				Writer.EndArray();	// softpackagereferences
			}
			Writer.EndObject();	// packagestoreentry
		}

		void WritePackageData(FCbWriter& Writer, const FOplogManifest::FOp::FPackageData& Data)
		{
			Writer.BeginObject();
			Writer.AddObjectId("id", Data.ID);
			Writer.AddInteger("size", Data.Size);
			Writer.AddInteger("rawsize", Data.RawSize);
			Writer.AddBinaryAttachment("data", Data.DataHash);
			Writer.AddString("filename", Data.Filename);
			Writer.EndObject();
		}

		void WriteBulkData(FCbWriter& Writer, const FOplogManifest::FOp::FBulkData& Data)
		{
			Writer.BeginObject();
			Writer.AddObjectId("id", Data.ID);
			Writer.AddString("type", Data.TypeStr);
			Writer.AddInteger("size", Data.Size);
			Writer.AddInteger("rawsize", Data.RawSize);
			Writer.AddBinaryAttachment("data", Data.DataHash);
			Writer.AddString("filename", Data.Filename);
			Writer.EndObject();
		}

		void WriteFile(FCbWriter& Writer, const FOplogManifest::FOp::FFile& Data)
		{
			Writer.BeginObject();
			Writer.AddObjectId("id", Data.ID);
			Writer.AddString("serverpath", Data.ServerPath);
			Writer.AddString("clientpath", Data.ClientPath);
			Writer.AddBinaryAttachment("data", Data.DataHash);
			Writer.EndObject();
		}

		void WriteMeta(FCbWriter& Writer, const FOplogManifest::FOp::FMeta& Data)
		{
			Writer.BeginObject();
			Writer.AddObjectId("id", Data.ID);
			Writer.AddString("name", Data.Name);
			Writer.AddBinaryAttachment("data", Data.DataHash);
			Writer.EndObject();
		}

		FOplogManifest GenerateSimpleTestManifest(FString Key, FString PackageStoreName, FIoHash PackageHash, int64 PackageSize, FUtf8String PackageDataFilename, TArray<int64> ImportExports = {})
		{
			FCbWriter CbWriter;
			CbWriter.BeginObject();
			CbWriter.BeginArray("ops");
			CbWriter.BeginObject();
			WriteOp(CbWriter, Key);
			WritePackageStoreEntry(CbWriter, PackageStoreName, 13, ImportExports, {});
			CbWriter.BeginArray("packagedata");
			FOplogManifest::FOp::FPackageData PackageData{
				.ID = FCbObjectId(FCbObjectId::ByteArray{10}),
				.Size = PackageSize,
				.RawSize = 32,
				.DataHash = PackageHash,
				.Filename = PackageDataFilename
			};
			WritePackageData(CbWriter, PackageData);
			CbWriter.EndArray();	//packagedata
			CbWriter.EndObject();
			CbWriter.EndArray();	// ops
			CbWriter.EndObject();
			FLoadOplogManifestResult LoadResult = LoadOplogManifestFromCompactBinary(CbWriter.Save().AsObject());
			CHECK(LoadResult.Result == FLoadOplogManifestResult::EStatus::Ok);
			return *LoadResult.Manifest;
		}

		FOplogManifest GenerateCustomTestManifest(TFunction<void(FCbWriter&)> WriteOpsArrayFn)
		{
			FCbWriter CbWriter;
			CbWriter.BeginObject();
			CbWriter.BeginArray("ops");
			WriteOpsArrayFn(CbWriter);
			CbWriter.EndArray();	// ops
			CbWriter.EndObject();
			FLoadOplogManifestResult LoadResult = LoadOplogManifestFromCompactBinary(CbWriter.Save().AsObject());
			CHECK(LoadResult.Result == FLoadOplogManifestResult::EStatus::Ok);
			return *LoadResult.Manifest;
		}

		FOplogManifest GenerateManifestWithMultipleOps(TArray<FString> OpKeys)
		{
			FCbWriter CbWriter;
			CbWriter.BeginObject();
			CbWriter.BeginArray("ops");
			for (const FString& Key : OpKeys)
			{
				CbWriter.BeginObject();
				WriteOp(CbWriter, Key);
				CbWriter.EndObject();
			}
			CbWriter.EndArray();	// ops
			CbWriter.EndObject();
			FLoadOplogManifestResult LoadResult = LoadOplogManifestFromCompactBinary(CbWriter.Save().AsObject());
			CHECK(LoadResult.Result == FLoadOplogManifestResult::EStatus::Ok);
			return *LoadResult.Manifest;
		}
	}

	TEST_CASE("ZenOplogUtils::Manifest::EmptyArchive", "[ZenOplogUtils]")
	{
		FCbObject EmptyObject;
		FLoadOplogManifestResult Result = LoadOplogManifestFromCompactBinary(EmptyObject);
		CHECK(Result.Result == FLoadOplogManifestResult::EStatus::ErrorMalformedCompactBinary);
	}

	TEST_CASE("ZenOplogUtils::Manifest::UnexpectedDataNoOps", "[ZenOplogUtils]")
	{
		FCbWriter CbWriter;
		CbWriter.BeginObject();
		CbWriter.BeginArray("Blah");
		CbWriter.EndArray();
		CbWriter.EndObject();
		FCbObject Cb = CbWriter.Save().AsObject();
		FLoadOplogManifestResult Result = LoadOplogManifestFromCompactBinary(Cb);
		CHECK(Result.Result == FLoadOplogManifestResult::EStatus::ErrorMalformedCompactBinary);
	}

	TEST_CASE("ZenOplogUtils::Manifest::EmptyOpsList", "[ZenOplogUtils]")
	{
		FCbWriter CbWriter;
		CbWriter.BeginObject();
		CbWriter.BeginArray("ops");
		CbWriter.EndArray();
		CbWriter.EndObject();
		FCbObject Cb = CbWriter.Save().AsObject();
		FLoadOplogManifestResult Result = LoadOplogManifestFromCompactBinary(Cb);
		CHECK(Result.Result == FLoadOplogManifestResult::EStatus::Ok);
	}

	TEST_CASE("ZenOplogUtils::Manifest::OpWithNoKey", "[ZenOplogUtils]")
	{
		FCbWriter CbWriter;
		CbWriter.BeginObject();
		CbWriter.BeginArray("ops");
		CbWriter.BeginObject();
		CbWriter.BeginObject("packagestoreentry");
		CbWriter.AddInteger("flags", 1);
		CbWriter.EndObject();
		CbWriter.EndObject();
		CbWriter.EndArray();
		CbWriter.EndObject();
		FCbObject Cb = CbWriter.Save().AsObject();
		FLoadOplogManifestResult Result = LoadOplogManifestFromCompactBinary(Cb);
		CHECK(Result.Result == FLoadOplogManifestResult::EStatus::ErrorMalformedCompactBinary);
	}

	TEST_CASE("ZenOplogUtils::Manifest::OpsWithDuplicateKey", "[ZenOplogUtils]")
	{
		FCbWriter CbWriter;
		CbWriter.BeginObject();
		CbWriter.BeginArray("ops");
		CbWriter.BeginObject();
		TestData::WriteOp(CbWriter, "ThisIsADuplicate");
		CbWriter.EndObject();
		CbWriter.BeginObject();
		TestData::WriteOp(CbWriter, "ThisIsOK");
		CbWriter.EndObject();
		CbWriter.BeginObject();
		TestData::WriteOp(CbWriter, "ThisIsADuplicate");
		CbWriter.EndObject();
		CbWriter.EndArray();
		CbWriter.EndObject();
		FCbObject Cb = CbWriter.Save().AsObject();
		FLoadOplogManifestResult Result = LoadOplogManifestFromCompactBinary(Cb);
		CHECK(Result.Result == FLoadOplogManifestResult::EStatus::Error);
	}

	TEST_CASE("ZenOplogUtils::Manifest::OpsWithUnexpectedData", "[ZenOplogUtils]")
	{
		FCbWriter CbWriter;
		CbWriter.BeginObject();
		CbWriter.BeginArray("ops");
		CbWriter.BeginObject();
		TestData::WriteOp(CbWriter, "Op1");
		CbWriter.AddString("somenewstring", "Blah");
		CbWriter.AddInteger("somenewinteger", 32);
		CbWriter.EndObject();
		CbWriter.BeginObject();
		TestData::WriteOp(CbWriter, "Op2");
		CbWriter.AddString("someotherstring", "Hmmm");
		CbWriter.BeginArray("SomeNewArray");
		CbWriter.BeginObject();
		CbWriter.AddBinaryAttachment("SomeNewAttachment", FIoHash(TEXT("aaaaaaaaabbbbbbbbbbddddddddddddddddddddd")));
		CbWriter.AddString("FilenameOrSomething", "Test");
		CbWriter.EndObject();
		CbWriter.EndArray();
		CbWriter.EndObject();
		CbWriter.EndArray();	// ops
		CbWriter.EndObject();
		FLoadOplogManifestResult Result = LoadOplogManifestFromCompactBinary(CbWriter.Save().AsObject());
		CHECK(Result.Result == FLoadOplogManifestResult::EStatus::Ok);
	}

	TEST_CASE("ZenOplogUtils::Manifest::OpAccessors", "[ZenOplogUtils]")
	{
		const FOplogManifest::FOp::FPackageData TestPackage
		{
			.ID = FCbObjectId(FCbObjectId::ByteArray{0}),
			.Size = 25,
			.RawSize = 64,
			.DataHash = FIoHash(TEXT("ed55912a82d1c88c4b0e73b1a0740277a82d732d")),
			.Filename = "TestFile1.uasset"
		};

		const FOplogManifest::FOp::FBulkData TestBulkData
		{
			.ID = FCbObjectId(FCbObjectId::ByteArray{1}),
			.TypeStr = "Mmap",
			.Size = 1024,
			.RawSize = 1400,
			.DataHash = FIoHash(TEXT("aaaabbbbccccdddd4b0e73b1a0740277a82d732d")),
			.Filename = "SomeFileWithLotsOfData.ubulk"
		};

		const FOplogManifest::FOp::FFile TestFile
		{
			.ID = FCbObjectId(FCbObjectId::ByteArray{2}),
			.ServerPath = "Path//On//Server",
			.ClientPath = "Path//On//Client",
			.DataHash = FIoHash(TEXT("aaaabbbbccccddddeeeeeeffffff0277a82d732d"))
		};

		const FOplogManifest::FOp::FMeta TestMeta
		{
			.ID = FCbObjectId(FCbObjectId::ByteArray{3}),
			.Name = "SomeName",
			.DataHash = FIoHash(TEXT("aaaabbabccccdddd2e6e2ef6ff2f2267a82d732d"))
		};

		const FIoHash ArtifactsHash(TEXT("d3d7a8a4c3a26b7b3a8234588213287a82d732da"));
		const FIoHash ImportExportHash(TEXT("11112225777322bbaa2627374237373782d7314b"));
		const FIoHash LogsHash(TEXT("cccccccccccbbbbbbbbbbbbbbbbbbbbbaaaaadde"));

		FCbWriter CbWriter;
		CbWriter.BeginObject();
		CbWriter.BeginArray("ops");
		CbWriter.BeginObject();
		TestData::WriteOp(CbWriter, "TestOp1", ArtifactsHash, ImportExportHash, LogsHash);
		TestData::WritePackageStoreEntry(CbWriter, "TestOp1Package", 42, { 1, 2, 3 }, { 8, 7, 6 });
		CbWriter.BeginArray("packagedata");
		TestData::WritePackageData(CbWriter, TestPackage);
		CbWriter.EndArray();	//packagedata
		CbWriter.BeginArray("bulkdata");
		TestData::WriteBulkData(CbWriter, TestBulkData);
		CbWriter.EndArray();	// bulkdata
		CbWriter.BeginArray("files");
		TestData::WriteFile(CbWriter, TestFile);
		CbWriter.EndArray();	// files
		CbWriter.BeginArray("meta");
		TestData::WriteMeta(CbWriter, TestMeta);
		CbWriter.EndArray();	// meta
		CbWriter.EndObject();
		CbWriter.EndArray();	// ops
		CbWriter.EndObject();
		FCbObject Cb = CbWriter.Save().AsObject();

		FLoadOplogManifestResult Result = LoadOplogManifestFromCompactBinary(Cb);
		CHECK(Result.Result == FLoadOplogManifestResult::EStatus::Ok);

		CHECK(Result.Manifest->Ops.Num() == 1);
		const FOplogManifest::FOp& TestOp = Result.Manifest->Ops[0];
		CHECK(TestOp.GetKey() == "TestOp1");
		CHECK(TestOp.GetArtifactHash() == ArtifactsHash);
		CHECK(TestOp.CookImportExport() == ImportExportHash);
		CHECK(TestOp.GetLogs() == LogsHash);

		FOplogManifest::FOp::FPackageStoreEntry PStoreEntry = TestOp.GetPackageStoreEntry();
		CHECK(PStoreEntry.Flags == 42);
		CHECK(PStoreEntry.PackageName == "TestOp1Package");
		CHECK(PStoreEntry.ImportedPackages == TArray<uint64>{1, 2, 3});
		CHECK(PStoreEntry.SoftPackageReferences == TArray<uint64>{8, 7, 6});

		TArray<FOplogManifest::FOp::FPackageData> Packages = TestOp.GetPackageDatas();
		CHECK(Packages.Num() == 1);
		CHECK(Packages[0].ID == TestPackage.ID);
		CHECK(Packages[0].Size == TestPackage.Size);
		CHECK(Packages[0].RawSize == TestPackage.RawSize);
		CHECK(Packages[0].DataHash == TestPackage.DataHash);
		CHECK(Packages[0].Filename == TestPackage.Filename);

		TArray<FOplogManifest::FOp::FBulkData> BulkDatas = TestOp.GetBulkDatas();
		CHECK(BulkDatas.Num() == 1);
		CHECK(BulkDatas[0].ID == TestBulkData.ID);
		CHECK(BulkDatas[0].TypeStr == TestBulkData.TypeStr);
		CHECK(BulkDatas[0].Size == TestBulkData.Size);
		CHECK(BulkDatas[0].RawSize == TestBulkData.RawSize);
		CHECK(BulkDatas[0].DataHash == TestBulkData.DataHash);
		CHECK(BulkDatas[0].Filename == TestBulkData.Filename);

		TArray<FOplogManifest::FOp::FFile> Files = TestOp.GetFiles();
		CHECK(Files.Num() == 1);
		CHECK(Files[0].ID == TestFile.ID);
		CHECK(Files[0].ServerPath == TestFile.ServerPath);
		CHECK(Files[0].ClientPath == TestFile.ClientPath);
		CHECK(Files[0].DataHash == TestFile.DataHash);

		TArray<FOplogManifest::FOp::FMeta> Metas = TestOp.GetMetas();
		CHECK(Metas.Num() == 1);
		CHECK(Metas[0].ID == TestMeta.ID);
		CHECK(Metas[0].Name == TestMeta.Name);
		CHECK(Metas[0].DataHash == TestMeta.DataHash);
	}

const FOplogDiffChangedOp* FindChangedOp(const FOplogDiffResults& Diff, FUtf8String OpKeyToFind)
{
	const FOplogDiffChangedOp* Changed = Diff.ChangedOpsWithSameOutput.FindByPredicate([&OpKeyToFind](const FOplogDiffChangedOp& Entry)
	{
		return Entry.Key == OpKeyToFind;
	});
	if (Changed != nullptr)
	{
		return Changed;
	}
	else
	{
		return Diff.ChangedOpsWithOutputDifferences.FindByPredicate([&OpKeyToFind](const FOplogDiffChangedOp& Entry)
		{
			return Entry.Key == OpKeyToFind;
		});
	}
};

bool ChangedOpPropertyExists(const FOplogDiffResults& Diff, FUtf8String OpKeyToFind, FUtf8String PropertyName)
{
	const FOplogDiffChangedOp* Changes = FindChangedOp(Diff, OpKeyToFind);
	if (Changes)
	{
		for (const FCbEntryDifference& Difference : Changes->Value)
		{
			if (Difference.PropertyName == PropertyName)
			{
				return true;
			}
		}
	}
	return false;
};

// Helper to do diff(manifest1,manifest2) and diff(manifest2,manifest1), with the same tests
void DiffBothDirections(const FOplogManifest& Manifest1, const FOplogManifest& Manifest2, TFunction<void(const FOplogDiffResults&)> CheckFn)
{
	FOplogDiffResults Diff1 = DiffManifests(Manifest1, Manifest2);
	FOplogDiffResults Diff2 = DiffManifests(Manifest2, Manifest1);
	CheckFn(Diff1);
	CheckFn(Diff2);
};

TEST_CASE("ZenOplogUtils::Diff::SameManifestIsEqual", "[ZenOplogUtils]")
{
	FOplogManifest SimpleManifest1 = TestData::GenerateSimpleTestManifest("TestOp1", 
		"TestPackage1", 
		FIoHash(TEXT("aaaaaabbbbbbbcccccccddddddeeeeeeffffffff")), 
		32, 
		"TestPackageData1", 
		{ 1, 2, 3 });
	DiffBothDirections(SimpleManifest1, SimpleManifest1, [](const FOplogDiffResults& Diff) 
	{
		CHECK(Diff.IdenticalOps.Num() == 1);
		CHECK(Diff.OpsMissingInManifest1.Num() == 0);
		CHECK(Diff.OpsMissingInManifest2.Num() == 0);
		CHECK(Diff.ChangedOpsWithSameOutput.Num() == 0);
		CHECK(Diff.ChangedOpsWithOutputDifferences.Num() == 0);
	});
}

TEST_CASE("ZenOplogUtils::Diff::MultipleSimilarOps", "[ZenOplogUtils]")
{
	// Same ops, different order
	FOplogManifest Manifest1 = TestData::GenerateManifestWithMultipleOps(
	{
		"SomeOp",
		"SomeOtherOp",
		"YetAnotherOp",
		"ReallyTheresMoreOps"
	});
	FOplogManifest Manifest2 = TestData::GenerateManifestWithMultipleOps(
	{
		"YetAnotherOp",
		"SomeOtherOp",
		"ReallyTheresMoreOps",
		"SomeOp"
	});
	DiffBothDirections(Manifest1, Manifest2, [](const FOplogDiffResults& Diff) 
	{
		CHECK(Diff.IdenticalOps.Num() == 4);
		CHECK(Diff.OpsMissingInManifest1.Num() == 0);
		CHECK(Diff.OpsMissingInManifest2.Num() == 0);
		CHECK(Diff.ChangedOpsWithSameOutput.Num() == 0);
		CHECK(Diff.ChangedOpsWithOutputDifferences.Num() == 0);
	});
}

TEST_CASE("ZenOplogUtils::Diff::DifferentOps", "[ZenOplogUtils]")
{
	FOplogManifest Manifest1 = TestData::GenerateManifestWithMultipleOps(
	{
		"Op1",
		"Op2",
		"Op3",
		"Op4",
		"Op5"
	});
	FOplogManifest Manifest2 = TestData::GenerateManifestWithMultipleOps(
	{
		"Op1",
		"OpA",
		"Op3",
		"OpB",
		"Op5"
	});
	FOplogDiffResults Diff = DiffManifests(Manifest1, Manifest2);
	CHECK(Diff.IdenticalOps.Num() == 3);
	CHECK(Diff.OpsMissingInManifest1.Num() == 2);
	CHECK(Diff.OpsMissingInManifest1.Contains("OpA"));
	CHECK(Diff.OpsMissingInManifest1.Contains("OpB"));
	CHECK(Diff.OpsMissingInManifest2.Num() == 2);
	CHECK(Diff.OpsMissingInManifest2.Contains("Op2"));
	CHECK(Diff.OpsMissingInManifest2.Contains("Op4"));
	CHECK(Diff.ChangedOpsWithSameOutput.Num() == 0);
	CHECK(Diff.ChangedOpsWithOutputDifferences.Num() == 0);
}

TEST_CASE("ZenOplogUtils::Diff::DifferentPackageName", "[ZenOplogUtils]")
{
	FOplogManifest SimpleManifest1 = TestData::GenerateSimpleTestManifest("TestOp1", 
		"TestPackage1", 
		FIoHash(TEXT("aaaaaabbbbbbbcccccccddddddeeeeeeffffffff")),
		32, 
		"TestPackageData1", 
		{ 1, 2, 3 });
	FOplogManifest SimpleManifestDifferentPackageName = TestData::GenerateSimpleTestManifest("TestOp1", 
		"SomePackageName", 
		FIoHash(TEXT("aaaaaabbbbbbbcccccccddddddeeeeeeffffffff")), 
		32, 
		"TestPackageData1", 
		{ 1, 2, 3 });
	DiffBothDirections(SimpleManifest1, SimpleManifestDifferentPackageName, [](const FOplogDiffResults& Diff) 
	{
		CHECK(Diff.IdenticalOps.Num() == 0);
		CHECK(Diff.OpsMissingInManifest1.Num() == 0);
		CHECK(Diff.OpsMissingInManifest2.Num() == 0);
		CHECK(Diff.ChangedOpsWithSameOutput.Num() == 1);
		CHECK(Diff.ChangedOpsWithOutputDifferences.Num() == 0);
		CHECK(FindChangedOp(Diff, "TestOp1"));
		CHECK(ChangedOpPropertyExists(Diff, "TestOp1", "packagestoreentry/packagename"));
	});
}

TEST_CASE("ZenOplogUtils::Diff::DifferentImportExportArraySize", "[ZenOplogUtils]")
{
	FOplogManifest SimpleManifest1 = TestData::GenerateSimpleTestManifest("TestOp1", 
		"TestPackage1", 
		FIoHash(TEXT("aaaaaabbbbbbbcccccccddddddeeeeeeffffffff")), 
		32, 
		"TestPackageData1", 
		{ 1, 2, 3 });
	FOplogManifest SimpleManifestDifferentImportExportSize = TestData::GenerateSimpleTestManifest("TestOp1", 
		"TestPackage1", 
		FIoHash(TEXT("aaaaaabbbbbbbcccccccddddddeeeeeeffffffff")), 
		32, 
		"TestPackageData1", 
		{ 6 });
	DiffBothDirections(SimpleManifest1, SimpleManifestDifferentImportExportSize, [](const FOplogDiffResults& Diff) 
	{
		CHECK(Diff.IdenticalOps.Num() == 0);
		CHECK(Diff.OpsMissingInManifest1.Num() == 0);
		CHECK(Diff.OpsMissingInManifest2.Num() == 0);
		CHECK(Diff.ChangedOpsWithSameOutput.Num() == 1);
		CHECK(Diff.ChangedOpsWithOutputDifferences.Num() == 0);
		CHECK(FindChangedOp(Diff, "TestOp1"));
		CHECK(ChangedOpPropertyExists(Diff, "TestOp1", "packagestoreentry/importedpackageids"));
	});
}

TEST_CASE("ZenOplogUtils::Diff::ManyDifferences", "[ZenOplogUtils]")
{
	FOplogManifest SimpleManifest1 = TestData::GenerateSimpleTestManifest("TestOp1", 
		"TestPackage1", 
		FIoHash(TEXT("aaaaaabbbbbbbcccccccddddddeeeeeeffffffff")), 
		32, 
		"TestPackageData1", 
		{ 1, 2, 3 });
	FOplogManifest SimpleManifestAllDifference = TestData::GenerateSimpleTestManifest("TestOp1", 
		"Huh", 
		FIoHash(TEXT("1111222233334444ccccddddddeeeeeeffffffff")), 
		111, 
		"Whatt", 
		{ 4, 2, 5 });
	DiffBothDirections(SimpleManifest1, SimpleManifestAllDifference, [](const FOplogDiffResults& Diff) 
	{
		CHECK(Diff.IdenticalOps.Num() == 0);
		CHECK(Diff.OpsMissingInManifest1.Num() == 0);
		CHECK(Diff.OpsMissingInManifest2.Num() == 0);
		CHECK(Diff.ChangedOpsWithSameOutput.Num() == 0);
		CHECK(Diff.ChangedOpsWithOutputDifferences.Num() == 1);
		CHECK(FindChangedOp(Diff, "TestOp1"));
		CHECK(Diff.ChangedOpsWithOutputDifferences[0].Value.Num() == 6);
		CHECK(ChangedOpPropertyExists(Diff, "TestOp1", "packagestoreentry/packagename"));
		CHECK(ChangedOpPropertyExists(Diff, "TestOp1", "packagestoreentry/importedpackageids/0"));
		CHECK(ChangedOpPropertyExists(Diff, "TestOp1", "packagestoreentry/importedpackageids/2"));
		CHECK(ChangedOpPropertyExists(Diff, "TestOp1", "packagedata/0/filename"));
		CHECK(ChangedOpPropertyExists(Diff, "TestOp1", "packagedata/0/size"));
		CHECK(ChangedOpPropertyExists(Diff, "TestOp1", "packagedata/0/data"));
	});
}

TEST_CASE("ZenOplogUtils::Diff::DataMissingInOps", "[ZenOplogUtils]")
{
	auto OpsWithArtifactsOrImportExports = [](FCbWriter& Writer)
	{
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op1", FIoHash(TEXT("aaaabbbbccccdddd4b0e73b1a0740277a82d732d")));
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op2", {}, FIoHash(TEXT("eeeeeeeffffffffaaaaaabbbbbbaaaaaaaa2222d")));
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op3", FIoHash(TEXT("6666666677777777778888888888844444411111")), FIoHash(TEXT("1111111112222222222233333333444444444444")));
		Writer.EndObject();
	};
	auto OpsWithMissingArtifactsOrImportExports = [](FCbWriter& Writer)
	{
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op1", {});
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op2", {}, {});
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op3", {}, {});
		Writer.EndObject();
	};
	FOplogManifest ManifestWithArtifactsOrImports = TestData::GenerateCustomTestManifest(OpsWithArtifactsOrImportExports);
	FOplogManifest ManifestWithMissingData = TestData::GenerateCustomTestManifest(OpsWithMissingArtifactsOrImportExports);
	DiffBothDirections(ManifestWithArtifactsOrImports, ManifestWithMissingData, [](const FOplogDiffResults& Diff) 
	{
		CHECK(Diff.IdenticalOps.Num() == 0);
		CHECK(Diff.OpsMissingInManifest1.Num() == 0);
		CHECK(Diff.OpsMissingInManifest2.Num() == 0);
		CHECK(Diff.ChangedOpsWithSameOutput.Num() == 3);
		CHECK(Diff.ChangedOpsWithOutputDifferences.Num() == 0);
		CHECK(ChangedOpPropertyExists(Diff, "Op1", "meta.cook.artifacts"));
		CHECK(ChangedOpPropertyExists(Diff, "Op2", "meta.cook.importexport"));
		CHECK(ChangedOpPropertyExists(Diff, "Op3", "meta.cook.artifacts"));
		CHECK(ChangedOpPropertyExists(Diff, "Op3", "meta.cook.importexport"));
	});
}

TEST_CASE("ZenOplogUtils::Diff::OpsWithMatchingPackages", "[ZenOplogUtils]")
{
	auto OpsWithPackages = [](FCbWriter& Writer)
	{
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op1");
		Writer.BeginArray("packagedata");
		TestData::WritePackageData(Writer, FOplogManifest::FOp::FPackageData{
			.ID = FCbObjectId(FCbObjectId::ByteArray{0}),
			.Size = 20,
			.RawSize = 32,
			.DataHash = FIoHash(TEXT("aaaaaabbbbbbbcccccccddddddeeeeeeffffffff")),
			.Filename = "Pack1"
			});
		Writer.EndArray();
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op2");
		Writer.BeginArray("packagedata");
		TestData::WritePackageData(Writer, FOplogManifest::FOp::FPackageData{
			.ID = FCbObjectId(FCbObjectId::ByteArray{1}),
			.Size = 30,
			.RawSize = 40,
			.DataHash = FIoHash(TEXT("aa1aaa3bbb5bbc7ccc8cddd3dde1eeeeffffffff")),
			.Filename = "Pack2"
			});
		TestData::WritePackageData(Writer, FOplogManifest::FOp::FPackageData{
			.ID = FCbObjectId(FCbObjectId::ByteArray{2}),
			.Size = 40,
			.RawSize = 50,
			.DataHash = FIoHash(TEXT("aaaaaabb1bb33c3cccc3d3dd6d6ee7e7f7ffffff")),
			.Filename = "Pack3"
			});
		Writer.EndArray();
		Writer.EndObject();
	};
	FOplogManifest ManifestWithPackageEntries = TestData::GenerateCustomTestManifest(OpsWithPackages);
	FOplogManifest OtherManifestWithPackageEntries = TestData::GenerateCustomTestManifest(OpsWithPackages);
	DiffBothDirections(ManifestWithPackageEntries, OtherManifestWithPackageEntries, [](const FOplogDiffResults& Diff)
	{
		CHECK(Diff.IdenticalOps.Num() == 2);
		CHECK(Diff.OpsMissingInManifest1.Num() == 0);
		CHECK(Diff.OpsMissingInManifest2.Num() == 0);
		CHECK(Diff.ChangedOpsWithSameOutput.Num() == 0);
		CHECK(Diff.ChangedOpsWithOutputDifferences.Num() == 0);
	});
}

TEST_CASE("ZenOplogUtils::Diff::OpsWithDifferentBulkDatas", "[ZenOplogUtils]")
{
	auto OpWithBulkData = [](FCbWriter& Writer)
	{
		const FOplogManifest::FOp::FBulkData TestBulkData1
		{
			.ID = FCbObjectId(FCbObjectId::ByteArray{0}),
			.TypeStr = "Mmap",
			.Size = 580,
			.RawSize = 1400,
			.DataHash = FIoHash(TEXT("aaaabbbbccccdddd4b0e73b1a0740277a82d732d")),
			.Filename = "SomeFileWithLotsOfData.ubulk"
		};
		const FOplogManifest::FOp::FBulkData TestBulkData2
		{
			.ID = FCbObjectId(FCbObjectId::ByteArray{1}),
			.TypeStr = "Mmap",
			.Size = 31,
			.RawSize = 20,
			.DataHash = FIoHash(TEXT("eeeeefffffffaaaaaabbb3b1a07402aaaaa11111")),
			.Filename = "AnotherFile.ubulk"
		};
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op1");
		Writer.BeginArray("bulkdata");
		TestData::WriteBulkData(Writer, TestBulkData1);
		TestData::WriteBulkData(Writer, TestBulkData2);
		Writer.EndArray();
		Writer.EndObject();
	};
	auto OpWithChangedBulkData = [](FCbWriter& Writer)
	{
		const FOplogManifest::FOp::FBulkData TestBulkData1
		{
			.ID = FCbObjectId(FCbObjectId::ByteArray{0}),
			.TypeStr = "Mmap",
			.Size = 580,
			.RawSize = 1400,
			.DataHash = FIoHash(TEXT("aaaabbbbccccdddd4b0e73b1a0740277a82d732d")),
			.Filename = "SomeFileWithLotsOfData.ubulk"
		};
		const FOplogManifest::FOp::FBulkData TestBulkData2
		{
			.ID = FCbObjectId(FCbObjectId::ByteArray{8}),
			.TypeStr = "SomethingElse",
			.Size = 20,
			.RawSize = 20,
			.DataHash = FIoHash(TEXT("eeeeefff1234aaaaaabbb3b1a07402aaaaa11111")),
			.Filename = "AnotherFileButNotTheSame.ubulk"
		};
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op1");
		Writer.BeginArray("bulkdata");
		TestData::WriteBulkData(Writer, TestBulkData1);
		TestData::WriteBulkData(Writer, TestBulkData2);
		Writer.EndArray();
		Writer.EndObject();
	};
	FOplogManifest Manifest1 = TestData::GenerateCustomTestManifest(OpWithBulkData);
	FOplogManifest Manifest2 = TestData::GenerateCustomTestManifest(OpWithChangedBulkData);
	DiffBothDirections(Manifest1, Manifest2, [](const FOplogDiffResults& Diff) 
	{
		CHECK(Diff.IdenticalOps.Num() == 0);
		CHECK(Diff.OpsMissingInManifest1.Num() == 0);
		CHECK(Diff.OpsMissingInManifest2.Num() == 0);
		CHECK(Diff.ChangedOpsWithSameOutput.Num() == 0);
		CHECK(Diff.ChangedOpsWithOutputDifferences.Num() == 1);
		CHECK(ChangedOpPropertyExists(Diff, "Op1", "bulkdata/1/id"));
		CHECK(ChangedOpPropertyExists(Diff, "Op1", "bulkdata/1/type"));
		CHECK(ChangedOpPropertyExists(Diff, "Op1", "bulkdata/1/size"));
		CHECK(ChangedOpPropertyExists(Diff, "Op1", "bulkdata/1/data"));
		CHECK(ChangedOpPropertyExists(Diff, "Op1", "bulkdata/1/filename"));
	});
}

TEST_CASE("ZenOplogUtils::Diff::OpsWithDifferentFiles", "[ZenOplogUtils]")
{
	const FOplogManifest::FOp::FFile TestFile1
	{
		.ID = FCbObjectId(FCbObjectId::ByteArray{0}),
		.ServerPath = "Path//On//Server//1",
		.ClientPath = "Path//On//Client//1",
		.DataHash = FIoHash(TEXT("aaaabbbbccccddddeeeeeeffffff0277a82d732d"))
	};
	const FOplogManifest::FOp::FFile TestFile2
	{
		.ID = FCbObjectId(FCbObjectId::ByteArray{1}),
		.ServerPath = "Path//On//Server//2",
		.ClientPath = "Path//On//Client//2",
		.DataHash = FIoHash(TEXT("aaaabb3bc666666666666e11ffff0277a82d732d"))
	};
	const FOplogManifest::FOp::FFile TestFile3
	{
		.ID = FCbObjectId(FCbObjectId::ByteArray{2}),
		.ServerPath = "Path//On//Server//3",
		.ClientPath = "Path//On//Client//3",
		.DataHash = FIoHash(TEXT("aaaa1111111cddddeeee11ffffff0277a82d732d"))
	};
	const FOplogManifest::FOp::FFile TestFile4
	{
		.ID = FCbObjectId(FCbObjectId::ByteArray{3}),
		.ServerPath = "Path//On//Server",
		.ClientPath = "Path//On//Client",
		.DataHash = FIoHash(TEXT("aaaab555555555555eee333333333333a82d732d"))
	};
	auto OpWithFiles1 = [&](FCbWriter& Writer)
	{
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op1");
		Writer.BeginArray("files");
		TestData::WriteFile(Writer, TestFile1);
		TestData::WriteFile(Writer, TestFile2);
		Writer.EndArray();
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op2");
		Writer.BeginArray("files");
		TestData::WriteFile(Writer, TestFile3);
		Writer.EndArray();
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op3");
		Writer.BeginArray("files");
		TestData::WriteFile(Writer, TestFile1);
		TestData::WriteFile(Writer, TestFile3);
		TestData::WriteFile(Writer, TestFile4);
		Writer.EndArray();
		Writer.EndObject();
	};
	auto OpWithFiles2 = [&](FCbWriter& Writer)
	{
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op1");
		Writer.BeginArray("files");
		TestData::WriteFile(Writer, TestFile3);
		Writer.EndArray();
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op2");
		Writer.BeginArray("files");
		TestData::WriteFile(Writer, TestFile1);
		Writer.EndArray();
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op3");
		Writer.BeginArray("files");
		TestData::WriteFile(Writer, TestFile1);
		TestData::WriteFile(Writer, TestFile2);
		TestData::WriteFile(Writer, TestFile3);
		TestData::WriteFile(Writer, TestFile4);
		Writer.EndArray();
		Writer.EndObject();
	};
	FOplogManifest Manifest1 = TestData::GenerateCustomTestManifest(OpWithFiles1);
	FOplogManifest Manifest2 = TestData::GenerateCustomTestManifest(OpWithFiles2);
	DiffBothDirections(Manifest1, Manifest2, [](const FOplogDiffResults& Diff) 
	{
		CHECK(Diff.IdenticalOps.Num() == 0);
		CHECK(Diff.OpsMissingInManifest1.Num() == 0);
		CHECK(Diff.OpsMissingInManifest2.Num() == 0);
		CHECK(Diff.ChangedOpsWithSameOutput.Num() == 0);
		CHECK(Diff.ChangedOpsWithOutputDifferences.Num() == 3);
		CHECK(ChangedOpPropertyExists(Diff, "Op1", "files"));
		CHECK(ChangedOpPropertyExists(Diff, "Op2", "files/0/id"));
		CHECK(ChangedOpPropertyExists(Diff, "Op2", "files/0/serverpath"));
		CHECK(ChangedOpPropertyExists(Diff, "Op2", "files/0/clientpath"));
		CHECK(ChangedOpPropertyExists(Diff, "Op2", "files/0/data"));
		CHECK(ChangedOpPropertyExists(Diff, "Op3", "files"));
	});
}

TEST_CASE("ZenOplogUtils::Diff::DiffUnexpectedData", "[ZenOplogUtils]")
{
	auto OpsWithUnexpectedData1 = [](FCbWriter& Writer)
	{
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op1");
		Writer.AddString("somenewstring", "Blah");
		Writer.AddInteger("somenewinteger", 32);
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op2");
		Writer.BeginArray("SomeNewArray");
		Writer.BeginObject();
		Writer.AddBinaryAttachment("SomeNewAttachment", FIoHash(TEXT("aaaaaaaaabbbbbbbbbbddddddddddddddddddddd")));
		Writer.AddString("FilenameOrSomething", "Test");
		Writer.EndObject();
		Writer.EndArray();
		Writer.EndObject();
	};
	auto OpsWithUnexpectedData2 = [](FCbWriter& Writer)
	{
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op1");
		Writer.AddString("somenewstring", "Blah");
		Writer.AddInteger("somenewinteger", 32);
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "Op2");
		Writer.AddString("someotherstring", "Eh?");
		Writer.AddString("adifferentstring", "Wat");
		Writer.BeginArray("SomeNewArray");
		Writer.BeginObject();
		Writer.AddBinaryAttachment("SomeNewAttachment", FIoHash(TEXT("eeeeeeeeeeffffffffffffaaaaaaaaaaaaaddddd")));
		Writer.AddString("FilenameOrSomething", "TestingThisThing");
		Writer.EndObject();
		Writer.EndArray();
		Writer.EndObject();
	};
	FOplogManifest Manifest1 = TestData::GenerateCustomTestManifest(OpsWithUnexpectedData1);
	FOplogManifest Manifest2 = TestData::GenerateCustomTestManifest(OpsWithUnexpectedData2);
	DiffBothDirections(Manifest1, Manifest2, [](const FOplogDiffResults& Diff) 
	{
		CHECK(Diff.IdenticalOps.Num() == 1);
		CHECK(Diff.OpsMissingInManifest1.Num() == 0);
		CHECK(Diff.OpsMissingInManifest2.Num() == 0);
		CHECK(Diff.ChangedOpsWithSameOutput.Num() == 1);
		CHECK(Diff.ChangedOpsWithOutputDifferences.Num() == 0);
		CHECK(ChangedOpPropertyExists(Diff, "Op2", "someotherstring"));
		CHECK(ChangedOpPropertyExists(Diff, "Op2", "SomeNewArray/0/SomeNewAttachment"));
		CHECK(ChangedOpPropertyExists(Diff, "Op2", "SomeNewArray/0/FilenameOrSomething"));
		CHECK(ChangedOpPropertyExists(Diff, "Op2", "adifferentstring"));
	});
}

TEST_CASE("ZenOplogUtils::Diff::Sorting", "[ZenOplogUtils]")
{
	auto Ops1 = [](FCbWriter& Writer)
	{
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpA");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpD");
		Writer.AddString("SomeValue", "Blah");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpC");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpE");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpB");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpF");
		Writer.AddString("SomeValue", "Blah2");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpH");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpG");
		Writer.EndObject();
	};
	auto Ops2 = [](FCbWriter& Writer)
	{
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpC");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpA");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpD");
		Writer.AddString("SomeValue", "Eh");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpF");
		Writer.AddString("SomeValue", "Hmm");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpH");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpG");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpJ");
		Writer.EndObject();
		Writer.BeginObject();
		TestData::WriteOp(Writer, "OpI");
		Writer.EndObject();
	};
	FOplogManifest Manifest1 = TestData::GenerateCustomTestManifest(Ops1);
	FOplogManifest Manifest2 = TestData::GenerateCustomTestManifest(Ops2);
	FOplogDiffResults Diff = DiffManifests(Manifest1, Manifest2);
	CHECK(Diff.IdenticalOps.Num() == 4);
	CHECK(Diff.OpsMissingInManifest1.Num() == 2);
	CHECK(Diff.OpsMissingInManifest2.Num() == 2);
	CHECK(Diff.ChangedOpsWithSameOutput.Num() == 2);
	CHECK(Diff.ChangedOpsWithOutputDifferences.Num() == 0);

	UE::SortDiffResults(Diff);
	CHECK(Algo::IsSorted(Diff.IdenticalOps));
	CHECK(Algo::IsSorted(Diff.OpsMissingInManifest1));
	CHECK(Algo::IsSorted(Diff.OpsMissingInManifest2));
	CHECK(Algo::IsSortedBy(Diff.ChangedOpsWithSameOutput, &FOplogDiffChangedOp::Key));
	CHECK(Algo::IsSortedBy(Diff.ChangedOpsWithOutputDifferences, &FOplogDiffChangedOp::Key));
}

// Temporarily disabled due to instability
// TEST_CASE("ZenOplogUtils::Diff::Helpers", "[ZenOplogUtils]")
// {
// 	auto OpWithArrays1 = [](FCbWriter& Writer)
// 	{
// 		Writer.BeginObject();
// 		TestData::WriteOp(Writer, "Op1");
// 		Writer.BeginArray("testarray");
// 		Writer.BeginObject();
// 		Writer.AddBinaryAttachment("SomeNewAttachment", FIoHash(TEXT("aaaaaaaaabbbbbbbbbbddddddddddddddddddddd")));
// 		Writer.AddString("FilenameOrSomething", "Test");
// 		Writer.EndObject();
// 		Writer.BeginObject();
// 		Writer.AddBinaryAttachment("SomeNewAttachment", FIoHash(TEXT("bbbbbbbbbbbccccccccccccccddddddddeeeeeee")));
// 		Writer.AddString("FilenameOrSomething", "AnotherTest");
// 		Writer.EndObject();
// 		Writer.EndArray();
// 		Writer.EndObject();
// 	};
// 	auto OpWithArrays2 = [](FCbWriter& Writer)
// 	{
// 		Writer.BeginObject();
// 		TestData::WriteOp(Writer, "Op1");
// 		Writer.BeginArray("testarray");
// 		Writer.BeginObject();
// 		Writer.AddBinaryAttachment("SomeNewAttachment", FIoHash(TEXT("aaaaaaaaabbbbbbbbbbddddddddddddddddddddd")));
// 		Writer.AddString("FilenameOrSomething", "DifferentValue");
// 		Writer.EndObject();
// 		Writer.BeginObject();
// 		Writer.AddBinaryAttachment("SomeNewAttachment", FIoHash(TEXT("bbbbbbbbbbbccccccccccccccddddddddeeeeeee")));
// 		Writer.AddString("FilenameOrSomething", "AnotherTest");
// 		Writer.EndObject();
// 		Writer.EndArray();
// 		Writer.EndObject();
// 	};
// 	FOplogManifest Manifest1 = TestData::GenerateCustomTestManifest(OpWithArrays1);
// 	FOplogManifest Manifest2 = TestData::GenerateCustomTestManifest(OpWithArrays2);
// 	FOplogDiffResults Diff = DiffManifests(Manifest1, Manifest2);
// 	CHECK(Diff.IdenticalOps.Num() == 0);
// 	CHECK(Diff.OpsMissingInManifest1.Num() == 0);
// 	CHECK(Diff.OpsMissingInManifest2.Num() == 0);
// 	CHECK(Diff.ChangedOpsWithSameOutput.Num() == 1);
// 	CHECK(Diff.ChangedOpsWithOutputDifferences.Num() == 0);
// 	CHECK(Diff.ChangedOpsWithSameOutput[0].Value.Num() == 1);
// 	CHECK(OpDiffContainsArrayValueChange(Diff.ChangedOpsWithSameOutput[0], "testarray", "FilenameOrSomething"));
// }

//TEST_CASE("ZenOplogUtils::Diff::DiffChangeHelper", "[ZenOplogUtils]")
//{
//	FOplogDiffChangedOp ChangedOp;
//	ChangedOp.Key = "Test";
//	ChangedOp.Value.Emplace(FCbEntryDifference{ .PropertyName = "Array1" });
//	ChangedOp.Value.Emplace(FCbEntryDifference{ .PropertyName = "Array2/0/Prop1" });
//	ChangedOp.Value.Emplace(FCbEntryDifference{ .PropertyName = "Array2/1/Prop2" });
//	ChangedOp.Value.Emplace(FCbEntryDifference{ .PropertyName = "Array2/2/SomeObject/Prop3" });
//	ChangedOp.Value.Emplace(FCbEntryDifference{ .PropertyName = "SomeObject/Array3/0/Prop4" });
//	ChangedOp.Value.Emplace(FCbEntryDifference{ .PropertyName = "SomeObject/Array4/0/SomeObject/Prop5" });
//	ChangedOp.Value.Emplace(FCbEntryDifference{ .PropertyName = "Array5/NotAnArrayEntry" });
//	ChangedOp.Value.Emplace(FCbEntryDifference{ .PropertyName = "Array5/NotAnArrayEntry/AlsoNotAnArrayEntry" });
//	CHECK(OpDiffContainsArrayValueChange(ChangedOp, "Array1", "SomeProperty"));
//	CHECK(OpDiffContainsArrayValueChange(ChangedOp, "Array2", "Prop1"));
//	CHECK(OpDiffContainsArrayValueChange(ChangedOp, "Array2", "Prop2"));
//	CHECK(!OpDiffContainsArrayValueChange(ChangedOp, "Array5", "NotAnArrayEntry"));
//	CHECK(!OpDiffContainsArrayValueChange(ChangedOp, "Array5", "AlsoNotAnArrayEntry"));
//	CHECK(!OpDiffContainsArrayValueChange(ChangedOp, "Array2", "Prop3"));
//	CHECK(OpDiffContainsArrayValueChange(ChangedOp, "Array2", "SomeObject/Prop3"));
//	CHECK(!OpDiffContainsArrayValueChange(ChangedOp, "Array3", "Prop4"));
//	CHECK(OpDiffContainsArrayValueChange(ChangedOp, "SomeObject/Array3", "Prop4"));
//	CHECK(!OpDiffContainsArrayValueChange(ChangedOp, "SomeObject/Array4", "Prop5"));
//	CHECK(OpDiffContainsArrayValueChange(ChangedOp, "SomeObject/Array4", "SomeObject/Prop5"));
//}

} // UE
#endif // WITH_EDITORONLY_DATA

#endif // WITH_LOW_LEVEL_TESTS
